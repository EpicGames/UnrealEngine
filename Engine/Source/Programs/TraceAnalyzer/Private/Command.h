// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE
{
namespace TraceAnalyzer
{

struct FCommand
{
	const TCHAR* Name;
	const TCHAR* Description;
	int32 (*Main)(int32, TCHAR const* const*);
};

} // namespace TraceAnalyzer
} // namespace UE
