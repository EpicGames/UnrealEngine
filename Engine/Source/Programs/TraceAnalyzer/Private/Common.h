// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceAnalyzer
#include "Io.h"

// TraceAnalysis
#include "Trace/DataStream.h"
#include "Trace/StoreClient.h"

namespace UE
{
namespace TraceAnalyzer
{

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FFileDataStream : public UE::Trace::IInDataStream
{
	~FFileDataStream()
	{
		CloseFile(Handle);
	}

	virtual int32 Read(void* Data, uint32 Size) override
	{
		return FileRead(Handle, Data, Size);
	}

	FileHandle Handle;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

inline TUniquePtr<UE::Trace::IInDataStream> GetDataStream(int ArgC, TCHAR const* const* ArgV)
{
	FileHandle Input = -1;

	if (ArgC < 2)
	{
		FileHandle StdIn = GetStdIn();
		if (!IsTty(StdIn))
		{
			Input = StdIn;
		}
	}
	else if (ArgC != 3)
	{
		Input = OpenFile(ArgV[1], false);
	}

	if (Input != -1)
	{
		FFileDataStream* Stream = new FFileDataStream();
		Stream->Handle = Input;
		return TUniquePtr<UE::Trace::IInDataStream>(Stream);
	}

	if (ArgC != 3)
	{
		return nullptr;
	}

	uint32 StorePort = FCString::Strtoui64(ArgV[1], nullptr, 10);
	if (TUniquePtr<UE::Trace::FStoreClient> StoreClient = UE::Trace::FStoreClient::Create(TEXT("127.0.0.1"), StorePort))
	{
		uint32 TraceId = FCString::Strtoui64(ArgV[2], nullptr, 10);
		return StoreClient->ReadTrace(TraceId);
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceAnalyzer
} // namespace UE
