// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceAnalyzer.h"

#include "Containers/Map.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformTime.h"
#include "Templates/UniquePtr.h"

// TraceAnalysis
#include "Trace/Analysis.h"
#include "Trace/Analyzer.h"
#include "Trace/DataStream.h"
#include "TraceAnalysisDebug.h"

// TraceAnalyzer
#include "Command.h"
#include "Io.h"
#include "TextSerializer.h"
#include "TraceAnalyzer.h"

#include <type_traits>

namespace UE
{
namespace TraceAnalyzer
{

////////////////////////////////////////////////////////////////////////////////////////////////////

// See \Engine\Source\Runtime\TraceLog\Public\Trace\Detail\Protocols\Protocol6.h
enum class EEventFlags : uint8
{
	Important   = 1 << 0,
	MaybeHasAux = 1 << 1,
	NoSync      = 1 << 2,
	Definition  = 1 << 3,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

static uint64 Decode7bit(const uint8*& BufferPtr)
{
	uint64 Value = 0;
	uint64 ByteIndex = 0;
	bool HasMoreBytes;
	do
	{
		uint8 ByteValue = *BufferPtr++;
		HasMoreBytes = ByteValue & 0x80;
		Value |= uint64(ByteValue & 0x7f) << (ByteIndex * 7);
		++ByteIndex;
	} while (HasMoreBytes);
	return Value;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static uint32 GetThreadIdField(const UE::Trace::IAnalyzer::FOnEventContext& Context, const ANSICHAR* FieldName = "ThreadId")
{
	// Trace analysis was changed to be able to provide a suitable id. Prior to
	// this users of Trace would send along their own thread ids. For backwards
	// compatibility we'll bias field thread ids to avoid collision with Trace's.
	static const uint32 Bias = 0x70000000;
	uint32 ThreadId = Context.EventData.GetValue<uint32>(FieldName, 0);
	ThreadId |= ThreadId ? Bias : Context.ThreadInfo.GetId();
	return ThreadId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FConvertToTextAnalyzer
////////////////////////////////////////////////////////////////////////////////////////////////////

class FConvertToTextAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FConvertToTextAnalyzer(FileHandle InHandle);
	virtual ~FConvertToTextAnalyzer();

private:
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual void OnVersion(uint32 InTransportVersion, uint32 InProtocolVersion) override;
	virtual bool OnNewEvent(uint16 RouteId, const FEventTypeInfo& TypeInfo) override;
	virtual bool OnEvent(uint16 RouteId, EStyle, const FOnEventContext& Context) override;
	void DefineTimer(uint32 SpecId, const TCHAR* TimerName);

	void WriteArrayHex(FTextSerializer& Serializer, TConstArrayView<uint8> ArrayView, uint32 MaxCount) const;

	template <typename ValueType>
	void WriteArray(FTextSerializer& Serializer, const TArrayReader<ValueType>& Reader, uint32 MaxCount) const;

	void ConvertToTextEventBatch(const FOnEventContext& Context, FTextSerializer& Serializer, const uint8* BufferPtr, uint32 BufferSize);
	void ConvertToTextEventBatchV2(const FOnEventContext& Context, FTextSerializer& Serializer, const uint8* BufferPtr, uint32 BufferSize);
	void PrintUnknownTimers(FTextSerializer& Serializer);
	void PrintEventStats(FTextSerializer& Serializer);

public:
	bool bNoNewEventLog = false;
	bool bNoEventLog = false;
	bool bNoAnalysisStats = false;
	bool bNoEventStats = false;

private:
	FileHandle Handle;

	TUniquePtr<FTextSerializer> TextSerializer;

	uint32 TransportVersion = 4;
	uint32 ProtocolVersion = 7;

	TMap<uint32, FString> TimerMap;
	TMap<uint32, uint64> ThreadMap;

	uint64 NumNewEvents = 0;
	uint64 NumEvents = 0;
	uint64 NumEnterScopeEvents = 0;
	uint64 NumLeaveScopeEvents = 0;
	uint64 NumCpuBatches = 0;
	uint64 NumBeginCpuEvents = 0;
	uint64 NumEndCpuEvents = 0;
	uint64 EventsTotalSize = 0;

	struct FEventStat
	{
		const FEventTypeInfo* TypeInfo = nullptr;
		uint64 Count = 0;
		uint64 ScopedCount = 0;
		uint64 Bytes = 0;
		uint64 ScopedBytes = 0;
		uint32 MinSize = 0xFFFFFFFF;
		uint32 MaxSize = 0;
	};

	TMap<int32, FEventStat> EventStats;

	uint64 AnalysisBeginTimestamp = 0;

	enum class EKnownEvent : uint32
	{
		Invalid,

		Trace_NewEvent,
		Trace_Timing,
		Trace_ThreadTiming,
		Trace_ThreadInfo,
		Trace_ThreadGroupBegin,
		Trace_ThreadGroupEnd,
		Trace_ChannelAnnounce, // "$Trace", deprecated in 4.26
		Trace_ChannelAnnounce2, // "Trace" new in 4.26

		Diagnostics_Session,
		Diagnostics_Session2,

		Misc_CreateThread,
		Misc_SetThreadGroup,
		Misc_BeginThreadGroupScope,
		Misc_BookmarkSpec,
		Misc_Bookmark,
		Misc_RegionBegin,
		Misc_RegionEnd,
		Misc_BeginGameFrame, // deprecated
		Misc_EndGameFrame, // deprecated
		Misc_BeginRenderFrame, // deprecated
		Misc_EndRenderFrame, // deprecated
		Misc_BeginFrame,
		Misc_EndFrame,

		Logging_LogCategory,
		Logging_LogMessageSpec,
		Logging_LogMessage,

		PlatformFile_BeginOpen,
		PlatformFile_EndOpen,
		PlatformFile_BeginRead,
		PlatformFile_EndRead,
		PlatformFile_BeginClose,
		PlatformFile_EndClose,

		LoadTime_ClassInfo,
		LoadTime_NewAsyncPackage,

		GpuProfiler_EventSpec,

		CpuProfiler_EventSpec,
		CpuProfiler_EventBatch, // deprecated
		CpuProfiler_EndCapture, // deprecated
		CpuProfiler_EventBatchV2,
		CpuProfiler_EndCaptureV2,

		Stats_Spec,

		Counters_Spec,

		CsvProfiler_RegisterCategory,
		CsvProfiler_DefineInlineStat,
		CsvProfiler_DefineDeclaredStat,
		CsvProfiler_Metadata,
		CsvProfiler_BeginCapture,

		Memory_Marker,

		LLM_TagValue,
		LLM_TagsSpec,
		LLM_TrackerSpec,

		SlateTrace_AddWidget,

		Count
	};

	static const uint32 InvalidEventId = (uint32)-1;

	typedef TFunction<void(const FOnEventContext& Context, FTextSerializer& Serializer)> KnownEventCallback;

	struct FKnownEvent
	{
		uint32 Id = InvalidEventId;
		KnownEventCallback Callback = nullptr;
		KnownEventCallback AttachmentCallback = nullptr;
	};
	FKnownEvent KnownEvents[(int)EKnownEvent::Count];

	static const uint32 MaxKnownEventId = 0xFFFF;
	EKnownEvent KnownEventMap[MaxKnownEventId + 1]; // index == EventId --> EKnownEvent

	bool IsKnownEvent(uint32 EventId, EKnownEvent KnownEvent) const
	{
		return EventId == KnownEvents[(int)KnownEvent].Id;
	}

	void RegisterCallbacksForKnownEvents();
	void RegisterAttachmentCallbacksForKnownEvents();
};

////////////////////////////////////////////////////////////////////////////////////////////////////

FConvertToTextAnalyzer::FConvertToTextAnalyzer(FileHandle InHandle)
	: Handle(InHandle)
{
	if (Handle < 0)
	{
		TextSerializer = MakeUnique<FStdoutTextSerializer>();
	}
	else
	{
		TextSerializer = MakeUnique<FFileTextSerializer>(Handle);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FConvertToTextAnalyzer::~FConvertToTextAnalyzer()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FConvertToTextAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	AnalysisBeginTimestamp = FPlatformTime::Cycles64();

	if (TextSerializer)
	{
		TextSerializer->Append("BEGIN conv2text\n");
	}

	Context.InterfaceBuilder.RouteAllEvents(0, false);
	Context.InterfaceBuilder.RouteAllEvents(1, true);

	KnownEvents[(int)EKnownEvent::Invalid].Id = InvalidEventId;
	KnownEvents[(int)EKnownEvent::Trace_NewEvent].Id = 0;
	for (uint32 EventId = 0; EventId <= MaxKnownEventId; ++EventId)
	{
		KnownEventMap[EventId] = EKnownEvent::Invalid;
	}
	KnownEventMap[0] = EKnownEvent::Trace_NewEvent;

	RegisterCallbacksForKnownEvents();
	RegisterAttachmentCallbacksForKnownEvents();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FConvertToTextAnalyzer::OnAnalysisEnd()
{
	if (TextSerializer)
	{
		if (!bNoAnalysisStats)
		{
			TextSerializer->Append("\n");
			TextSerializer->Append("Stats (conv2text):\n");
			TextSerializer->Appendf("  New Events: % llu\n", NumNewEvents);
			TextSerializer->Appendf("  Events: %llu\n", NumEvents);
			TextSerializer->Appendf("  | Normal Events: %llu\n", NumEvents - NumEnterScopeEvents - NumLeaveScopeEvents);
			TextSerializer->Appendf("  | Scoped Events: %llu\n", NumEnterScopeEvents + NumLeaveScopeEvents);
			TextSerializer->Appendf("  | | Enter Scope Events: %llu\n", NumEnterScopeEvents);
			if (NumEnterScopeEvents == NumLeaveScopeEvents)
			{
				TextSerializer->Appendf("  | | Leave Scope Events: %llu\n", NumLeaveScopeEvents);
			}
			else if (NumEnterScopeEvents > NumLeaveScopeEvents)
			{
				TextSerializer->Appendf("  | | Leave Scope Events: %llu (-%llu)\n", NumLeaveScopeEvents, NumEnterScopeEvents - NumLeaveScopeEvents);
			}
			else // if (NumEnterScopeEvents < NumLeaveScopeEvents)
			{
				TextSerializer->Appendf("  | | Leave Scope Events: %llu (+%llu)\n", NumLeaveScopeEvents, NumLeaveScopeEvents - NumEnterScopeEvents);
			}
			if (!bNoEventLog)
			{
				TextSerializer->Appendf("  CPU Batches: %llu\n", NumCpuBatches);
				TextSerializer->Appendf("  CPU Events: %llu\n", NumBeginCpuEvents + NumEndCpuEvents);
				TextSerializer->Appendf("  | Begin Events: %llu\n", NumBeginCpuEvents);
				if (NumBeginCpuEvents == NumEndCpuEvents)
				{
					TextSerializer->Appendf("  |   End Events: %llu\n", NumEndCpuEvents);
				}
				else if (NumBeginCpuEvents > NumEndCpuEvents)
				{
					TextSerializer->Appendf("  |   End Events: %llu (-%llu)\n", NumEndCpuEvents, NumBeginCpuEvents - NumEndCpuEvents);
				}
				else // if (NumBeginCpuEvents < NumEndCpuEvents)
				{
					TextSerializer->Appendf("  |   End Events: %llu (+%llu)\n", NumEndCpuEvents, NumEndCpuEvents - NumBeginCpuEvents);
				}
				TextSerializer->Appendf("  CPU Timers: %d\n", TimerMap.Num());
			}
			TextSerializer->Appendf("  Total Event Size: %llu bytes\n", EventsTotalSize);

			PrintUnknownTimers(*TextSerializer);
		}

		if (!bNoEventStats)
		{
			TextSerializer->Append("\n");
			PrintEventStats(*TextSerializer);
		}

		TextSerializer->Append("\n");
		uint64 AnalysisDuration = FPlatformTime::Cycles64() - AnalysisBeginTimestamp;
		double Duration = (double)AnalysisDuration * FPlatformTime::GetSecondsPerCycle();
		if (Duration < 60.0)
		{
			TextSerializer->Appendf("END conv2text -- %.02fs\n", Duration);
		}
		else if (Duration < 120.0)
		{
			TextSerializer->Appendf("END conv2text -- %.02fs (1 minute %lli seconds)\n", Duration, (int64)Duration % 60);
		}
		else
		{
			TextSerializer->Appendf("END conv2text -- %.02fs (%lli minutes %lli seconds)\n", Duration, (int64)Duration / 60, (int64)Duration % 60);
		}

		TextSerializer->Commit();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FConvertToTextAnalyzer::PrintEventStats(FTextSerializer& Serializer)
{
	TArray<uint32> Ids;
	for (const auto Iter : EventStats)
	{
		Ids.Add(Iter.Key);
	}

	// Sort by Id.
	//Ids.Sort();

	// Sort by LoggerName/Name.
	Ids.Sort([&](const uint32& A, const uint32& B)
	{
		const FEventTypeInfo* TypeNameA = EventStats[A].TypeInfo;
		const FEventTypeInfo* TypeNameB = EventStats[B].TypeInfo;
		const ANSICHAR* LoggerNameA = TypeNameA ? TypeNameA->GetLoggerName() : "";
		const ANSICHAR* LoggerNameB = TypeNameB ? TypeNameB->GetLoggerName() : "";
		int32 ret = FCStringAnsi::Strcmp(LoggerNameA, LoggerNameB);
		if (ret == 0)
		{
			const ANSICHAR* EventNameA = TypeNameA ? TypeNameA->GetName() : "";
			const ANSICHAR* EventNameB = TypeNameB ? TypeNameB->GetName() : "";
			ret = FCStringAnsi::Strcmp(EventNameA, EventNameB);
		}
		return ret < 0;
	});

	uint64 TotalCount = 0;
	uint64 TotalSize = 0;
	Serializer.Append(" ID       COUNT       BYTES   AVSZ   FXSZ   MNSZ   MXSZ\n");
	for (int StatIndex = 0, StatCount = Ids.Num(); StatIndex < StatCount; ++StatIndex)
	{
		const FEventStat& Stat = *EventStats.Find(Ids[StatIndex]);

		const FEventTypeInfo* TypeInfo = Stat.TypeInfo;

		const uint32 FixedEventSize = TypeInfo ? TypeInfo->GetSize() : 0;
		const uint32 MinEventSize = (Stat.MinSize <= Stat.MaxSize) ? Stat.MinSize : 0;
		const uint32 MaxEventSize = Stat.MaxSize;

		const ANSICHAR* LoggerName = TypeInfo ? TypeInfo->GetLoggerName() : "";
		const ANSICHAR* EventName = TypeInfo ? TypeInfo->GetName() : "";

		if (Stat.Count > 0)
		{
			Serializer.Appendf("%3d %11llu %11llu %6u %6u %6u %6u %s.%s\n",
				Ids[StatIndex],
				Stat.Count,
				Stat.Bytes,
				Stat.Count != 0 ? uint32(Stat.Bytes / Stat.Count) : 0,
				FixedEventSize,
				MinEventSize,
				MaxEventSize,
				LoggerName,
				EventName);
			TotalCount += Stat.Count;
			TotalSize += Stat.Bytes;
		}
		if (Stat.ScopedCount > 0)
		{
			Serializer.Appendf("%3d %11llu %11llu %6u %6u %6u %6u %s.%s*\n",
				Ids[StatIndex],
				Stat.ScopedCount,
				Stat.ScopedBytes,
				Stat.ScopedCount != 0 ? uint32(Stat.ScopedBytes / Stat.ScopedCount) : 0,
				FixedEventSize,
				MinEventSize,
				MaxEventSize,
				LoggerName,
				EventName);
			TotalCount += Stat.ScopedCount;
			TotalSize += Stat.ScopedBytes;
		}
	}
	Serializer.Append("--------------------------------------------------------------------------------\n");
	Serializer.Appendf("    %11llu %11llu\n", TotalCount, TotalSize);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FConvertToTextAnalyzer::OnVersion(uint32 InTransportVersion, uint32 InProtocolVersion)
{
	TransportVersion = InTransportVersion;
	ProtocolVersion = InProtocolVersion;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FConvertToTextAnalyzer::RegisterCallbacksForKnownEvents()
{
	KnownEvents[(int)EKnownEvent::GpuProfiler_EventSpec].Callback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const auto& Name = EventData.GetArray<UTF16CHAR>("Name");
			auto NameTChar = StringCast<TCHAR>(Name.GetData(), Name.Num());
			FStringView NameView(NameTChar.Get(), NameTChar.Length());
			FString NameStr(NameView.GetData(), NameView.Len());
			Serializer.WriteAttributeString("#Name", TCHAR_TO_UTF8(*NameStr));
		};

	KnownEvents[(int)EKnownEvent::CpuProfiler_EventSpec].Callback =
		[this](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const uint32 SpecId = EventData.GetValue<uint32>("Id");

			FString Name;
			const TCHAR* TimerName = nullptr;
			if (EventData.GetString("Name", Name))
			{
				TimerName = *Name;
			}
			else
			{
				uint8 CharSize = EventData.GetValue<uint8>("CharSize");
				if (CharSize == sizeof(ANSICHAR))
				{
					check(EventData.GetAttachment() != nullptr);
					TimerName = StringCast<TCHAR>(reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment())).Get();
				}
				else if (CharSize == 0 || CharSize == sizeof(TCHAR)) // 0 for backwards compatibility
				{
					check(EventData.GetAttachment() != nullptr);
					TimerName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
				}
				if (TimerName)
				{
					Serializer.WriteAttributeString("@Name", TCHAR_TO_UTF8(TimerName));
				}
				else
				{
					Serializer.WriteAttributeString("@Name", "<invalid>");
				}
			}

			DefineTimer(SpecId, TimerName);
		};

	KnownEvents[(int)EKnownEvent::CpuProfiler_EventBatch].Callback =
	KnownEvents[(int)EKnownEvent::CpuProfiler_EndCapture].Callback =
		[this](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			TArrayView<const uint8> Out = EventData.GetArrayView<uint8>("Data");
			if (Out.GetData() != nullptr)
			{
				Serializer.BeginAttribute();
				Serializer.WriteKey("#Data");
				ConvertToTextEventBatch(Context, Serializer, Out.GetData(), Out.Num());
				Serializer.EndAttribute();
			}
			else
			{
				check(EventData.GetAttachment() != nullptr);
				Serializer.BeginAttribute();
				Serializer.WriteKey("Attached");
				ConvertToTextEventBatch(Context, Serializer, EventData.GetAttachment(), EventData.GetAttachmentSize());
				Serializer.EndAttribute();
			}
		};

	KnownEvents[(int)EKnownEvent::CpuProfiler_EventBatchV2].Callback =
	KnownEvents[(int)EKnownEvent::CpuProfiler_EndCaptureV2].Callback =
		[this](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			TArrayView<const uint8> Out = Context.EventData.GetArrayView<uint8>("Data");
			if (Out.GetData() != nullptr)
			{
				Serializer.BeginAttribute();
				Serializer.WriteKey("#Data");
				ConvertToTextEventBatchV2(Context, Serializer, Out.GetData(), Out.Num());
				Serializer.EndAttribute();
			}
		};

	//KnownEvents[(int)EKnownEvent::Misc_Bookmark].Callback =
	//KnownEvents[(int)EKnownEvent::Misc_RegionBegin].Callback =
	//KnownEvents[(int)EKnownEvent::Misc_RegionEnd].Callback =
	//KnownEvents[(int)EKnownEvent::Misc_BeginFrame].Callback =
	//KnownEvents[(int)EKnownEvent::Misc_EndFrame].Callback =
	//KnownEvents[(int)EKnownEvent::Logging_LogMessage].Callback =
	//KnownEvents[(int)EKnownEvent::PlatformFile_BeginOpen].Callback =
	//KnownEvents[(int)EKnownEvent::PlatformFile_EndOpen].Callback =
	//KnownEvents[(int)EKnownEvent::PlatformFile_BeginRead].Callback =
	//KnownEvents[(int)EKnownEvent::PlatformFile_EndRead].Callback =
	//KnownEvents[(int)EKnownEvent::PlatformFile_BeginClose].Callback =
	//KnownEvents[(int)EKnownEvent::PlatformFile_EndClose].Callback =
	//KnownEvents[(int)EKnownEvent::Memory_Marker].Callback =
	//KnownEvents[(int)EKnownEvent::LLM_TagValue].Callback =
	//KnownEvents[(int)EKnownEvent::SlateTrace_AddWidget].Callback =
	//	[](const FOnEventContext& Context, FTextSerializer& Serializer)
	//	{
	//		uint64 Cycle = Context.EventData.GetValue<uint64>("Cycle");
	//		double Time = Context.EventTime.AsSeconds(Cycle);
	//		Serializer.WriteAttributeFloat("#Time", Time);
	//	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FConvertToTextAnalyzer::RegisterAttachmentCallbacksForKnownEvents()
{
	KnownEvents[(int)EKnownEvent::Trace_ChannelAnnounce].AttachmentCallback =
	KnownEvents[(int)EKnownEvent::Trace_ChannelAnnounce2].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
			if (TypeInfo.GetFieldIndex("Name") < 0)
			{
				const ANSICHAR* ChannelName = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment());
				Serializer.WriteAttributeString("@ChannelName", ChannelName);
			}
		};

	KnownEvents[(int)EKnownEvent::Trace_ThreadInfo].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
			if (TypeInfo.GetFieldIndex("Name") < 0)
			{
				const ANSICHAR* ThreadName = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment());
				Serializer.WriteAttributeString("@Name", ThreadName);
			}
		};

	KnownEvents[(int)EKnownEvent::Diagnostics_Session].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const uint8 AppNameOffset = EventData.GetValue<uint8>("AppNameOffset");
			const uint8 CommandLineOffset = EventData.GetValue<uint8>("CommandLineOffset");

			const ANSICHAR* Platform = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment());
			Serializer.WriteAttributeString("@Platform", Platform, AppNameOffset);

			const ANSICHAR* AppName = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment() + AppNameOffset);
			Serializer.WriteAttributeString("@AppName", AppName, CommandLineOffset - AppNameOffset);

			const ANSICHAR* CommandLine = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment() + CommandLineOffset);
			Serializer.WriteAttributeString("@CommandLine", CommandLine, EventData.GetAttachmentSize() - CommandLineOffset);
		};

	KnownEvents[(int)EKnownEvent::Diagnostics_Session2].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			// hide attachment
		};

	KnownEvents[(int)EKnownEvent::Misc_CreateThread].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const TCHAR* ThreadName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
			Serializer.WriteAttributeString("@Name", TCHAR_TO_UTF8(ThreadName));
		};

	KnownEvents[(int)EKnownEvent::Misc_BeginGameFrame].AttachmentCallback =
	KnownEvents[(int)EKnownEvent::Misc_EndGameFrame].AttachmentCallback =
	KnownEvents[(int)EKnownEvent::Misc_BeginRenderFrame].AttachmentCallback =
	KnownEvents[(int)EKnownEvent::Misc_EndRenderFrame].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const uint8* BufferPtr = EventData.GetAttachment();
			uint64 CycleDiff = Decode7bit(BufferPtr);
			Serializer.WriteAttributeInteger("@CycleDiff", CycleDiff);
		};

	KnownEvents[(int)EKnownEvent::Trace_ThreadGroupBegin].AttachmentCallback =
	KnownEvents[(int)EKnownEvent::Misc_SetThreadGroup].AttachmentCallback =
	KnownEvents[(int)EKnownEvent::Misc_BeginThreadGroupScope].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
			if (TypeInfo.GetFieldIndex("Name") < 0)
			{
				const ANSICHAR* GroupName = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment());
				Serializer.WriteAttributeString("@GroupName", GroupName);
			}
		};

	KnownEvents[(int)EKnownEvent::Misc_BookmarkSpec].AttachmentCallback =
	KnownEvents[(int)EKnownEvent::Logging_LogMessageSpec].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
			if (TypeInfo.GetFieldIndex("FileName") < 0)
			{
				const ANSICHAR* File = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment());
				Serializer.WriteAttributeString("@File", File);

				const TCHAR* FormatString = reinterpret_cast<const TCHAR*>(EventData.GetAttachment() + strlen(File) + 1);
				Serializer.WriteAttributeString("@FormatString", TCHAR_TO_UTF8(FormatString));
			}
		};

	KnownEvents[(int)EKnownEvent::Logging_LogCategory].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
			if (TypeInfo.GetFieldIndex("Name") < 0)
			{
				const TCHAR* CategoryName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
				Serializer.WriteAttributeString("@CategoryName", TCHAR_TO_UTF8(CategoryName));
			}
		};

	KnownEvents[(int)EKnownEvent::PlatformFile_BeginOpen].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const TCHAR* ClassName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
			Serializer.WriteAttributeString("@File", TCHAR_TO_UTF8(ClassName));
		};

	KnownEvents[(int)EKnownEvent::LoadTime_ClassInfo].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
			if (TypeInfo.GetFieldIndex("Name") < 0)
			{
				const TCHAR* ClassName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
				Serializer.WriteAttributeString("@ClassName", TCHAR_TO_UTF8(ClassName));
			}
		};

	KnownEvents[(int)EKnownEvent::LoadTime_NewAsyncPackage].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const TCHAR* ClassName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
			Serializer.WriteAttributeString("@PackageName", TCHAR_TO_UTF8(ClassName));
		};

	KnownEvents[(int)EKnownEvent::CpuProfiler_EventSpec].AttachmentCallback =
		[this](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const uint32 SpecId = EventData.GetValue<uint32>("Id");

			FString Name;
			const TCHAR* TimerName = nullptr;
			if (EventData.GetString("Name", Name))
			{
				TimerName = *Name;
			}
			else
			{
				uint8 CharSize = EventData.GetValue<uint8>("CharSize");
				if (CharSize == sizeof(ANSICHAR))
				{
					check(EventData.GetAttachment() != nullptr);
					TimerName = StringCast<TCHAR>(reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment())).Get();
				}
				else if (CharSize == 0 || CharSize == sizeof(TCHAR)) // 0 for backwards compatibility
				{
					check(EventData.GetAttachment() != nullptr);
					TimerName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
				}
				if (TimerName)
				{
					Serializer.WriteAttributeString("@Name", TCHAR_TO_UTF8(TimerName));
				}
				else
				{
					Serializer.WriteAttributeString("@Name", "<invalid>");
				}
			}

			DefineTimer(SpecId, TimerName);
		};

	KnownEvents[(int)EKnownEvent::CpuProfiler_EventBatch].AttachmentCallback =
	KnownEvents[(int)EKnownEvent::CpuProfiler_EndCapture].AttachmentCallback =
		[this](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			check(EventData.GetAttachment() != nullptr);
			Serializer.BeginAttribute();
			Serializer.WriteKey("Attached");
			ConvertToTextEventBatch(Context, Serializer, EventData.GetAttachment(), EventData.GetAttachmentSize());
			Serializer.EndAttribute();
		};

	KnownEvents[(int)EKnownEvent::Stats_Spec].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
			if (TypeInfo.GetFieldIndex("Name") < 0)
			{
				const ANSICHAR* Name = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment());
				Serializer.WriteAttributeString("@Name", Name);

				const TCHAR* Description = reinterpret_cast<const TCHAR*>(EventData.GetAttachment() + strlen(Name) + 1);
				Serializer.WriteAttributeString("@Description", TCHAR_TO_UTF8(Description));
			}
		};

	KnownEvents[(int)EKnownEvent::Counters_Spec].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
			if (TypeInfo.GetFieldIndex("Name") < 0)
			{
				const TCHAR* Name = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
				Serializer.WriteAttributeString("@Name", TCHAR_TO_UTF8(Name));
			}
		};

	KnownEvents[(int)EKnownEvent::CsvProfiler_RegisterCategory].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
			if (TypeInfo.GetFieldIndex("Name") < 0)
			{
				const TCHAR* Name = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
				Serializer.WriteAttributeString("@Name", TCHAR_TO_UTF8(Name));
			}
		};

	KnownEvents[(int)EKnownEvent::CsvProfiler_DefineInlineStat].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
			if (TypeInfo.GetFieldIndex("Name") < 0)
			{
				const ANSICHAR* Name = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment());
				Serializer.WriteAttributeString("@Name", Name);
			}
		};

	KnownEvents[(int)EKnownEvent::CsvProfiler_DefineDeclaredStat].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
			if (TypeInfo.GetFieldIndex("Name") < 0)
			{
				const TCHAR* Name = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
				Serializer.WriteAttributeString("@Name", TCHAR_TO_UTF8(Name));
			}
		};

	KnownEvents[(int)EKnownEvent::CsvProfiler_Metadata].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const TCHAR* Key = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
			const TCHAR* Value = reinterpret_cast<const TCHAR*>(EventData.GetAttachment() + EventData.GetValue<uint16>("ValueOffset"));
			Serializer.WriteAttributeString("@Key", TCHAR_TO_UTF8(Key));
			Serializer.WriteAttributeString("@Value", TCHAR_TO_UTF8(Value));
		};

	KnownEvents[(int)EKnownEvent::CsvProfiler_BeginCapture].AttachmentCallback =
		[](const FOnEventContext& Context, FTextSerializer& Serializer)
		{
			const FEventData& EventData = Context.EventData;
			const TCHAR* Filename = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
			Serializer.WriteAttributeString("@Filename", TCHAR_TO_UTF8(Filename));
		};
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FConvertToTextAnalyzer::OnNewEvent(uint16 RouteId, const FEventTypeInfo& TypeInfo)
{
	++NumNewEvents;

	const uint32 EventId = TypeInfo.GetId();

	const ANSICHAR* LoggerName = TypeInfo.GetLoggerName();
	check(LoggerName != nullptr);

	const ANSICHAR* EventName = TypeInfo.GetName();
	check(EventName != nullptr);

	if (!bNoEventStats)
	{
		FEventStat* Stat = EventStats.Find(EventId);
		if (!Stat)
		{
			EventStats.Add(EventId, { &TypeInfo });
		}
	}

	#define BEGIN_KNOWN_LOGGER(_Name) \
		if (strcmp(LoggerName, _Name) == 0) \
		{

	#define KNOWN_EVENT(_Name,_Id) \
			if (strcmp(EventName, _Name) == 0) \
			{ \
				KnownEvents[(int)EKnownEvent::_Id].Id = EventId; \
				KnownEventMap[EventId] = EKnownEvent::_Id; \
				break; \
			}

	#define END_KNOWN_LOGGER() \
			break; \
		}

	if (!bNoEventLog)
	while (true)
	{
		BEGIN_KNOWN_LOGGER("$Trace")
			KNOWN_EVENT("NewEvent", Trace_NewEvent)
			KNOWN_EVENT("ThreadTiming", Trace_ThreadTiming)
			KNOWN_EVENT("ThreadInfo", Trace_ThreadInfo)
			KNOWN_EVENT("ThreadGroupBegin", Trace_ThreadGroupBegin)
			KNOWN_EVENT("ChannelAnnounce", Trace_ChannelAnnounce)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("Trace")
			KNOWN_EVENT("ChannelAnnounce", Trace_ChannelAnnounce2)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("Diagnostics")
			KNOWN_EVENT("Session", Diagnostics_Session)
			KNOWN_EVENT("Session2", Diagnostics_Session2)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("Misc")
			KNOWN_EVENT("CreateThread", Misc_CreateThread)
			KNOWN_EVENT("SetThreadGroup", Misc_SetThreadGroup)
			KNOWN_EVENT("BeginThreadGroupScope", Misc_BeginThreadGroupScope)
			KNOWN_EVENT("BookmarkSpec", Misc_BookmarkSpec)
			KNOWN_EVENT("Bookmark", Misc_Bookmark)
			KNOWN_EVENT("RegionBegin", Misc_RegionBegin)
			KNOWN_EVENT("RegionEnd", Misc_RegionEnd)
			KNOWN_EVENT("BeginGameFrame", Misc_BeginGameFrame)
			KNOWN_EVENT("EndGameFrame", Misc_EndGameFrame)
			KNOWN_EVENT("BeginRenderFrame", Misc_BeginRenderFrame)
			KNOWN_EVENT("EndRenderFrame", Misc_EndRenderFrame)
			KNOWN_EVENT("BeginFrame", Misc_BeginFrame)
			KNOWN_EVENT("EndFrame", Misc_EndFrame)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("Logging")
			KNOWN_EVENT("LogCategory", Logging_LogCategory)
			KNOWN_EVENT("LogMessageSpec", Logging_LogMessageSpec)
			KNOWN_EVENT("LogMessage", Logging_LogMessage)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("PlatformFile")
			KNOWN_EVENT("BeginOpen", PlatformFile_BeginOpen)
			KNOWN_EVENT("EndOpen", PlatformFile_EndOpen)
			KNOWN_EVENT("BeginRead", PlatformFile_BeginRead)
			KNOWN_EVENT("EndRead", PlatformFile_EndRead)
			KNOWN_EVENT("BeginClose", PlatformFile_BeginClose)
			KNOWN_EVENT("EndClose", PlatformFile_EndClose)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("LoadTime")
			KNOWN_EVENT("ClassInfo", LoadTime_ClassInfo)
			KNOWN_EVENT("NewAsyncPackage", LoadTime_NewAsyncPackage)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("GpuProfiler")
			KNOWN_EVENT("EventSpec", GpuProfiler_EventSpec)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("CpuProfiler")
			KNOWN_EVENT("EventSpec", CpuProfiler_EventSpec)
			KNOWN_EVENT("EventBatch", CpuProfiler_EventBatch)
			KNOWN_EVENT("EndCapture", CpuProfiler_EndCapture)
			KNOWN_EVENT("EventBatchV2", CpuProfiler_EventBatchV2)
			KNOWN_EVENT("EndCaptureV2", CpuProfiler_EndCaptureV2)
			END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("Stats")
			KNOWN_EVENT("Spec", Stats_Spec)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("Counters")
			KNOWN_EVENT("Spec", Counters_Spec)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("CsvProfiler")
			KNOWN_EVENT("RegisterCategory", CsvProfiler_RegisterCategory)
			KNOWN_EVENT("DefineInlineStat", CsvProfiler_DefineInlineStat)
			KNOWN_EVENT("DefineDeclaredStat", CsvProfiler_DefineDeclaredStat)
			KNOWN_EVENT("Metadata", CsvProfiler_Metadata)
			KNOWN_EVENT("BeginCapture", CsvProfiler_BeginCapture)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("Memory")
			KNOWN_EVENT("Marker", Memory_Marker)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("LLM")
			KNOWN_EVENT("TagValue", LLM_TagValue)
			KNOWN_EVENT("TagsSpec", LLM_TagsSpec)
			KNOWN_EVENT("TrackerSpec", LLM_TrackerSpec)
		END_KNOWN_LOGGER()

		BEGIN_KNOWN_LOGGER("SlateTrace")
			KNOWN_EVENT("AddWidget", SlateTrace_AddWidget)
		END_KNOWN_LOGGER()

		break;
	}

	#undef BEGIN_KNOWN_LOGGER
	#undef KNOWN_EVENT
	#undef END_KNOWN_LOGGER

	//////////////////////////////////////////////////
	// Filter

	if (bNoNewEventLog)
	{
		return true;
	}

	//////////////////////////////////////////////////

	FTextSerializer& Serializer = *TextSerializer;

	if (Serializer.IsWriteEventHeaderEnabled())
	{
		Serializer.BeginNewEventHeader();

		Serializer.WriteAttributeInteger("Id", TypeInfo.GetId());

		Serializer.WriteAttributeString("LoggerName", TypeInfo.GetLoggerName());

		Serializer.WriteAttributeString("Name", TypeInfo.GetName());

		Serializer.BeginAttribute();
		Serializer.WriteKey("Flags");
		Serializer.Append("\"");

		IAnalyzer::EEventFlags EventFlags = TypeInfo.GetFlags();
		if (TypeInfo.IsDefinition())
		{
			Serializer.Append("Definition|");
			EnumRemoveFlags(EventFlags, IAnalyzer::EEventFlags::Definition);
		}
		if (TypeInfo.IsImportant())
		{
			Serializer.Append("Important|");
			EnumRemoveFlags(EventFlags, IAnalyzer::EEventFlags::Important);
		}
		if (TypeInfo.MaybeHasAux())
		{
			Serializer.Append("MaybeHasAux|");
			EnumRemoveFlags(EventFlags, IAnalyzer::EEventFlags::MaybeHasAux);
		}
		if (TypeInfo.IsNoSync())
		{
			Serializer.Append("NoSync");
			EnumRemoveFlags(EventFlags, IAnalyzer::EEventFlags::NoSync);
		}
		else
		{
			Serializer.Append("Sync");
		}
		if ((uint32)EventFlags != 0)
		{
			Serializer.Appendf("|%u", (uint32)EventFlags);
		}
		Serializer.Append("\"");
		Serializer.EndAttribute();

		Serializer.EndNewEventHeader();
	}

	Serializer.BeginNewEventFields();

	for (int FieldIndex = 0, FieldCount = TypeInfo.GetFieldCount(); FieldIndex < FieldCount; ++FieldIndex)
	{
		Serializer.BeginField();

		const FEventFieldInfo& FieldInfo = *(TypeInfo.GetFieldInfo(FieldIndex));
		Serializer.WriteAttributeString("Name", FieldInfo.GetName());

		/** Offset from the start of the event to this field's data. */
		Serializer.WriteAttributeInteger("Offset", FieldInfo.GetOffset());

		/** The size of the field's data in bytes. */
		Serializer.WriteAttributeInteger("Size", FieldInfo.GetSize());

		/** Is this field signed? */
		Serializer.WriteAttributeBool("IsSigned", FieldInfo.IsSigned());

		/** Is this field an array-type field? */
		Serializer.WriteAttributeBool("IsArray", FieldInfo.IsArray());

		Serializer.BeginAttribute();
		/** What type of field is this? */
		Serializer.WriteKey("Type");

		using EType = UE::Trace::IAnalyzer::FEventFieldInfo::EType;
		switch (FieldInfo.GetType())
		{
		case EType::Integer:
		{
			Serializer.Append("Integer");
			if (FieldInfo.IsSigned())
			{
				switch (FieldInfo.GetTypeSize())
				{
				case 1: // int8
				case 2: // int16
				case 4: // int32
				case 8: // int64
					Serializer.Appendf(" (int%d%s)", 8 * FieldInfo.GetTypeSize(), FieldInfo.IsArray() ? "[]" : "");
					break;
				default:
					Serializer.Appendf(" (%d bytes/element)", FieldInfo.GetTypeSize());
				}
			}
			else
			{
				switch (FieldInfo.GetTypeSize())
				{
				case 1: // uint8
				case 2: // uint16
				case 4: // uint32
				case 8: // uint64
					Serializer.Appendf(" (uint%d%s)", 8 * FieldInfo.GetTypeSize(), FieldInfo.IsArray() ? "[]" : "");
					break;
				default:
					Serializer.Appendf(" (%d bytes/element)", FieldInfo.GetTypeSize());
				}
			}
			break;
		}
		case EType::Float: // Float32 or Float64
		{
			Serializer.Appendf("Float");
			switch (FieldInfo.GetTypeSize())
			{
			case 4: // float
				Serializer.Appendf(" (float%s)", FieldInfo.IsArray() ? "[]" : "");
				break;
			case 8: // double
				Serializer.Appendf(" (double%s)", FieldInfo.IsArray() ? "[]" : "");
				break;
			default:
				Serializer.Appendf(" (%d bytes/element)", FieldInfo.GetTypeSize());
			}
			break;
		}
		case EType::AnsiString:
			Serializer.Append("AnsiString");
			break;
		case EType::WideString:
			Serializer.Append("WideString");
			break;
		case EType::Reference8:
			Serializer.Append("Reference8");
			break;
		case EType::Reference16:
			Serializer.Append("Reference16");
			break;
		case EType::Reference32:
			Serializer.Append("Reference32");
			break;
		case EType::Reference64:
			Serializer.Append("Reference64");
			break;
		default:
			Serializer.WriteValueUInt32(static_cast<uint32>(FieldInfo.GetType()));
			Serializer.Appendf(" (%d bytes/element)", FieldInfo.GetTypeSize());
			break;
		}
		Serializer.EndAttribute();

		Serializer.EndField();
	}

	Serializer.EndNewEventFields();

	return Serializer.Commit();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FConvertToTextAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	const FEventData& EventData = Context.EventData;
	const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
	const uint32 EventId = TypeInfo.GetId();

	//////////////////////////////////////////////////
	// Update event stats.

	++NumEvents;
	if (Style == EStyle::EnterScope)
	{
		++NumEnterScopeEvents;
	}
	else if (Style == EStyle::LeaveScope)
	{
		++NumLeaveScopeEvents;
	}

	uint32 TotalEventSize = EventData.GetTotalSize(Style, Context, ProtocolVersion);

	if (!bNoEventStats)
	{
		FEventStat* Stat = EventStats.Find(EventId);
		if (Stat == nullptr)
		{
			if (TypeInfo.GetLoggerName()[0] == 0) // empty dispatch, expected on LeaveScope events
			{
				Stat = &EventStats.Add(0, { nullptr });
			}
			else
			{
				Stat = &EventStats.Add(EventId, { &TypeInfo });
				check(TypeInfo.GetLoggerName() != nullptr);
				check(TypeInfo.GetName() != nullptr);
			}
		}

		if (RouteId == 0)
		{
			Stat->Count++;
			Stat->Bytes += TotalEventSize;
		}
		else // if (RouteId == 1)
		{
			Stat->ScopedCount++;
			Stat->ScopedBytes += TotalEventSize;
		}

		Stat->MinSize = FMath::Min<uint16>(Stat->MinSize, TotalEventSize);
		Stat->MaxSize = FMath::Max<uint16>(Stat->MaxSize, TotalEventSize);
	}

	//////////////////////////////////////////////////
	// Filter 1
	// Allows only the specified type of events.

	//if (strcmp(TypeInfo.GetLoggerName(), "...") != 0 ||
	//	  strcmp(TypeInfo.GetName(), "...") != 0)
	//{
	//	return true;
	//}

	//if (!IsKnownEvent(EventId, EKnownEvent::...))
	//{
	//	return true;
	//}

	//////////////////////////////////////////////////
	// Filter 2
	// Allows all but the specified type of events.

	//if (strcmp(TypeInfo.GetLoggerName(), "...") == 0)
	//{
	//	if (strcmp(TypeInfo.GetName(), "...") == 0)
	//	{
	//		return true;
	//	}
	//}

	//if (IsKnownEvent(EventId, EKnownEvent::...))
	//{
	//	return true;
	//}

	//////////////////////////////////////////////////

	const uint32 FieldCount = (Style == EStyle::LeaveScope) ? 0 : TypeInfo.GetFieldCount();

	const uint32 FixedSize = TypeInfo.GetSize();
	const uint32 RawSize = EventData.GetRawSize();
	const uint32 AuxSize = EventData.GetAuxSize();
	const uint32 AttachmentSize = EventData.GetAttachmentSize();

	EventsTotalSize += TotalEventSize;

	const bool bHasAttachment = !TypeInfo.IsImportant() &&
								(AuxSize == 0) &&
								(RawSize > FixedSize) &&
								(AttachmentSize > 0) &&
								(EventData.GetAttachment() != nullptr);

	bool bEmitSizeStats = false;

	const uint32 ContextThreadId = Context.ThreadInfo.GetId();

	//////////////////////////////////////////////////

	if (bNoEventLog)
	{
		return true;
	}

	FTextSerializer& Serializer = *TextSerializer;

	Serializer.BeginEvent(ContextThreadId);

	if (RouteId == 1)
	{
		Serializer.BeginAttribute();
		if (Style == EStyle::EnterScope)
		{
			Serializer.Append("!EnterScope");
		}
		else if (Style == EStyle::LeaveScope)
		{
			Serializer.Append("!LeaveScope");
			check(FieldCount == 0);
			check(!bHasAttachment);
		}
		else if (Style == EStyle::Normal)
		{
			Serializer.Append("!Normal");
		}
		else
		{
			Serializer.Appendf("!%u", (uint32)Style);
		}
		Serializer.EndAttribute();
	}

	Serializer.WriteEventName(TypeInfo.GetLoggerName(), TypeInfo.GetName());

	//////////////////////////////////////////////////

	for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
	{
		const FEventFieldInfo& FieldInfo = *(TypeInfo.GetFieldInfo(FieldIndex));

		if (FieldInfo.IsArray())
		{
			if (IsKnownEvent(EventId, EKnownEvent::GpuProfiler_EventSpec) || // "Name" array
				IsKnownEvent(EventId, EKnownEvent::CpuProfiler_EventBatch) || // "Data" array
				IsKnownEvent(EventId, EKnownEvent::CpuProfiler_EndCapture) || // "Data" array
				IsKnownEvent(EventId, EKnownEvent::CpuProfiler_EventBatchV2) || // "Data" array
				IsKnownEvent(EventId, EKnownEvent::CpuProfiler_EndCaptureV2)) // "Data" array
			{
				continue;
			}
		}

		using EType = UE::Trace::IAnalyzer::FEventFieldInfo::EType;

		Serializer.BeginAttribute();
		Serializer.WriteKey(FieldInfo.GetName());
		if (FieldInfo.IsArray())
		{
			switch (FieldInfo.GetType())
			{
				case EType::Integer:
				{
#if 0
					// Probe element size...
					uint8 ElementSize = 0;
					{
						const TArrayReader<uint8>& Reader8 = EventData.GetArray<uint8>(FieldInfo.GetName());
						if (Reader8.GetData())
						{
							ElementSize = 1;
						}
						else
						{
							const TArrayReader<uint16>& Reader16 = EventData.GetArray<uint16>(FieldInfo.GetName());
							if (Reader16.GetData())
							{
								ElementSize = 2;
							}
							else
							{
								const TArrayReader<uint32>& Reader32 = EventData.GetArray<uint32>(FieldInfo.GetName());
								if (Reader16.GetData())
								{
									ElementSize = 4;
								}
								else
								{
									ElementSize = 8;
								}
							}
						}
					}
#endif

					if (FieldInfo.IsSigned())
					{
						switch (FieldInfo.GetTypeSize())
						{
							case 1: // int8
							{
								const TArrayReader<int8>& Reader = EventData.GetArray<int8>(FieldInfo.GetName());
								WriteArray<int8>(Serializer, Reader, 64);
								break;
							}
							case 2: // int16
							{
								const TArrayReader<int16>& Reader = EventData.GetArray<int16>(FieldInfo.GetName());
								WriteArray<int16>(Serializer, Reader, 32);
								break;
							}
							case 4: // int32
							{
								const TArrayReader<int32>& Reader = EventData.GetArray<int32>(FieldInfo.GetName());
								WriteArray<int32>(Serializer, Reader, 16);
								break;
							}
							case 8: // int64
							{
								const TArrayReader<int64>& Reader = EventData.GetArray<int64>(FieldInfo.GetName());
								WriteArray<int64>(Serializer, Reader, 8);
								break;
							}
							default:
							{
								const TArrayReader<uint8>& Reader = EventData.GetArray<uint8>(FieldInfo.GetName());
								WriteArrayHex(Serializer, TConstArrayView<uint8>((const uint8*)Reader.GetData(), (int32)Reader.Num()), 32);
							}
						} // switch (FieldInfo.GetTypeSize())
					}
					else // unsigned
					{
						switch (FieldInfo.GetTypeSize())
						{
							case 1: // uint8
							{
								const TArrayReader<uint8>& Reader = EventData.GetArray<uint8>(FieldInfo.GetName());
								//WriteArray<uint8>(Serializer, Reader, 64);
								WriteArrayHex(Serializer, TConstArrayView<uint8>((const uint8*)Reader.GetData(), (int32)Reader.Num()), 32);
								break;
							}
							case 2: // uint16
							{
								const TArrayReader<uint16>& Reader = EventData.GetArray<uint16>(FieldInfo.GetName());
								WriteArray<uint16>(Serializer, Reader, 32);
								break;
							}
							case 4: // uint32
							{
								const TArrayReader<uint32>& Reader = EventData.GetArray<uint32>(FieldInfo.GetName());
								WriteArray<uint32>(Serializer, Reader, 16);
								break;
							}
							case 8: // uint64
							{
								const TArrayReader<uint64>& Reader = EventData.GetArray<uint64>(FieldInfo.GetName());
								WriteArray<uint64>(Serializer, Reader, 8);
								break;
							}
							default:
							{
								const TArrayReader<uint8>& Reader = EventData.GetArray<uint8>(FieldInfo.GetName());
								WriteArrayHex(Serializer, TConstArrayView<uint8>((const uint8*)Reader.GetData(), (int32)Reader.Num()), 32);
							}
						} // switch (FieldInfo.GetTypeSize())
					}
					break;
				}

				case EType::Float: // Float32 or Float64
				{
					switch (FieldInfo.GetTypeSize())
					{
						case 4: // float
						{
							const TArrayReader<float>& Reader = EventData.GetArray<float>(FieldInfo.GetName());
							WriteArray<float>(Serializer, Reader, 8);
							break;
						}
						case 8: // double
						{
							const TArrayReader<double>& Reader = EventData.GetArray<double>(FieldInfo.GetName());
							WriteArray<double>(Serializer, Reader, 8);
							break;
						}
						default:
						{
							const TArrayReader<uint8>& Reader = EventData.GetArray<uint8>(FieldInfo.GetName());
							WriteArrayHex(Serializer, TConstArrayView<uint8>((const uint8*)Reader.GetData(), (int32)Reader.Num()), 32);
						}
					} // switch (FieldInfo.GetTypeSize())
					break;
				}

				case EType::AnsiString:
				{
					FAnsiStringView Value;
					EventData.GetString(FieldInfo.GetName(), Value);
					Serializer.Appendf("\"%.*s\"", Value.Len(), Value.GetData());
					break;
				}

				case EType::WideString:
				{
					FString Value;
					EventData.GetString(FieldInfo.GetName(), Value);
					Serializer.WriteValueString(TCHAR_TO_UTF8(*Value));
					break;
				}

				default:
				{
					// error
					Serializer.Append("<UnknownArrayType>");
				}
			}
		}
		else
		{
			switch (FieldInfo.GetType())
			{
				case EType::Integer:
				{
					if (FieldInfo.IsSigned())
					{
						const int64 Value = EventData.GetValue<int64>(FieldInfo.GetName());
						if (strcmp(FieldInfo.GetName(), "Cycle") == 0)
						{
							Serializer.WriteValueUInt64(uint64(Value));
							Serializer.AppendChar('(');
							double Time = Context.EventTime.AsSeconds(uint64(Value));
							Serializer.WriteValueTime(Time);
							Serializer.AppendChar(')');
						}
						else
						{
							Serializer.WriteValueInt64Auto(Value);
						}
					}
					else // unsigned
					{
						const uint64 Value = EventData.GetValue<uint64>(FieldInfo.GetName());
						if (strcmp(FieldInfo.GetName(), "Cycle") == 0)
						{
							Serializer.WriteValueUInt64(Value);
							Serializer.AppendChar('(');
							double Time = Context.EventTime.AsSeconds(Value);
							Serializer.WriteValueTime(Time);
							Serializer.AppendChar(')');
						}
						else
						{
							Serializer.WriteValueUInt64Auto(Value);
						}
					}
					break;
				}

				case EType::Float: // Float32 or Float64
				{
					switch (FieldInfo.GetTypeSize())
					{
						case 4: // float
						{
							const float Value = EventData.GetValue<float>(FieldInfo.GetName());
							Serializer.WriteValueFloatAuto(Value);
							break;
						}
						case 8: // double
						{
							const double Value = EventData.GetValue<double>(FieldInfo.GetName());
							Serializer.WriteValueDoubleAuto(Value);
							break;
						}
						default:
						{
							// error
							Serializer.Append("<UnknownFloatTypeSize>");
						}
					} // switch (FieldInfo.GetTypeSize())
					break;
				}

				case EType::Reference8:
				{
					check(FieldInfo.GetSize() == 1);
					UE::Trace::FEventRef8 RefValue = EventData.GetReferenceValue<uint8>(FieldIndex);
					Serializer.WriteValueReference(RefValue);
					break;
				}

				case EType::Reference16:
				{
					check(FieldInfo.GetSize() == 2);
					UE::Trace::FEventRef16 RefValue = EventData.GetReferenceValue<uint16>(FieldIndex);
					Serializer.WriteValueReference(RefValue);
					break;
				}

				case EType::Reference32:
				{
					check(FieldInfo.GetSize() == 4);
					UE::Trace::FEventRef32 RefValue = EventData.GetReferenceValue<uint32>(FieldIndex);
					Serializer.WriteValueReference(RefValue);
					break;
				}

				case EType::Reference64:
				{
					check(FieldInfo.GetSize() == 8);
					UE::Trace::FEventRef64 RefValue = EventData.GetReferenceValue<uint64>(FieldIndex);
					Serializer.WriteValueReference(RefValue);
					break;
				}

				default:
				{
					// error
					Serializer.Append("<UnknownType>");
				}
			}
		}
		Serializer.EndAttribute();
	}

	//////////////////////////////////////////////////

	check(EventId <= MaxKnownEventId);
	EKnownEvent KnownEvent = KnownEventMap[EventId];
	check((int)KnownEvent < (int)EKnownEvent::Count);
	const FKnownEvent& Event = KnownEvents[(int)KnownEvent];

	if (Event.Callback)
	{
		Event.Callback(Context, Serializer);
	}

	if (bHasAttachment)
	{
		if (Event.AttachmentCallback)
		{
			Event.AttachmentCallback(Context, Serializer);
		}
		else
		{
			Serializer.WriteAttributeBinary("Attached", EventData.GetAttachment(), AttachmentSize);
		}
	}

	if (bEmitSizeStats)
	{
		Serializer.AppendChar('\n');
		Serializer.BeginAttributeSet();
		Serializer.BeginAttribute();
		Serializer.Appendf("> SIZE Fix=%u Aux=%u Att=%u Raw=%u Total=%u", FixedSize, AuxSize, AttachmentSize, RawSize, TotalEventSize);
		Serializer.EndAttribute();
	}

	//////////////////////////////////////////////////

	Serializer.EndEvent();

	return Serializer.Commit();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FConvertToTextAnalyzer::WriteArrayHex(FTextSerializer& Serializer, TConstArrayView<uint8> ArrayView, uint32 MaxCount) const
{
	// Write array as 0x[00 01 ... FF]
	Serializer.Append("0x");
	Serializer.BeginArray();

	const uint32 ArrayCount = ArrayView.Num();
	const uint32 ActualCount = FMath::Min(ArrayCount, MaxCount);
	for (uint32 ArrayIndex = 0; ArrayIndex < ActualCount; ++ArrayIndex)
	{
		if (ArrayIndex != 0)
		{
			Serializer.NextArrayElement();
		}
		const uint8 Value = ArrayView.GetData()[ArrayIndex];
		Serializer.Appendf("%02X", uint32(Value));
	}
	if (ActualCount != ArrayCount)
	{
		Serializer.Appendf(" ... | %u bytes", ArrayCount);
	}

	Serializer.EndArray();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename ValueType>
void FConvertToTextAnalyzer::WriteArray(FTextSerializer& Serializer, const TArrayReader<ValueType>& Reader, uint32 MaxCount) const
{
	TConstArrayView<ValueType> ArrayView(Reader.GetData(), Reader.Num());

	Serializer.BeginArray();

	const uint32 ArrayCount = Reader.Num();
	const uint32 ActualCount = FMath::Min(ArrayCount, MaxCount);
	for (uint32 ArrayIndex = 0; ArrayIndex < ActualCount; ++ArrayIndex)
	{
		if (ArrayIndex != 0)
		{
			Serializer.NextArrayElement();
		}

		const ValueType Value = ArrayView[ArrayIndex];
		const ValueType Value2 = Reader[ArrayIndex];
		check(Value == Value2);

		     if constexpr (std::is_same_v<ValueType, uint8>)  { Serializer.WriteValueUInt8(Value); }
		else if constexpr (std::is_same_v<ValueType, uint16>) { Serializer.WriteValueUInt16(Value); }
		else if constexpr (std::is_same_v<ValueType, uint32>) { Serializer.WriteValueUInt64Auto(uint64(Value)); }
		else if constexpr (std::is_same_v<ValueType, uint64>) { Serializer.WriteValueUInt64Auto(Value); }
		else if constexpr (std::is_same_v<ValueType, int8>)   { Serializer.WriteValueInt8(Value); }
		else if constexpr (std::is_same_v<ValueType, int16>)  { Serializer.WriteValueInt16(Value); }
		else if constexpr (std::is_same_v<ValueType, int32>)  { Serializer.WriteValueInt64Auto(int64(Value)); }
		else if constexpr (std::is_same_v<ValueType, int64>)  { Serializer.WriteValueInt64Auto(Value); }
		else if constexpr (std::is_same_v<ValueType, float>)  { Serializer.WriteValueFloatAuto(Value); }
		else if constexpr (std::is_same_v<ValueType, double>) { Serializer.WriteValueDoubleAuto(Value); }
	}
	if (ActualCount != ArrayCount)
	{
		Serializer.Appendf(" ... | %u elements", ArrayCount);
	}

	Serializer.EndArray();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FConvertToTextAnalyzer::ConvertToTextEventBatch(const FOnEventContext& Context, FTextSerializer& Serializer, const uint8* BufferPtr, uint32 BufferSize)
{
	++NumCpuBatches;

	const FEventData& EventData = Context.EventData;

	Serializer.Append("[");

	bool bFirstElement = true;

	const uint32 ThreadId = GetThreadIdField(Context);
	uint64* LastCyclePtr = ThreadMap.Find(ThreadId);
	if (!LastCyclePtr)
	{
		LastCyclePtr = &ThreadMap.Add(ThreadId, 0);
	}
	uint64 LastCycle = *LastCyclePtr;

	const uint8* BufferEnd = BufferPtr + BufferSize;
	while (BufferPtr < BufferEnd)
	{
		if (!bFirstElement)
		{
			Serializer.AppendChar(' ');
		}
		else
		{
			bFirstElement = false;
		}

		uint64 DecodedCycle = Decode7bit(BufferPtr);

		uint64 Cycle = DecodedCycle >> 1;
		uint64 ActualCycle = Cycle + LastCycle;
		LastCycle = ActualCycle;

		const bool bIsAbsoluteTimestamp = (Cycle > 100000000000ull);

		if (DecodedCycle & 1ull)
		{
			++NumBeginCpuEvents;

			//if (DecodedCycle >> 1 == 16)
			//{
			//	Serializer.Append("!!!");
			//	BufferPtr = BufferEnd;
			//	break;
			//}

			bool bIsUnknownSpecId = false;
			uint32 SpecId = Decode7bit(BufferPtr);
			FString* NamePtr = TimerMap.Find(SpecId);
			if (!NamePtr)
			{
				// SpecId not yet declared (no "EventSpec" event for this id)!
				DefineTimer(SpecId, nullptr);
				bIsUnknownSpecId = true;
			}

			//Serializer.WriteInteger("B", ActualCycle);
			if (bIsAbsoluteTimestamp)
			{
				Serializer.Append("*B=");
				Serializer.WriteValueInt64(Cycle);
				Serializer.AppendChar('(');
				double Time = Context.EventTime.AsSeconds(Cycle);
				Serializer.WriteValueTime(Time);
				Serializer.AppendChar(')');
			}
			else
			{
				Serializer.Append("B=");
				Serializer.WriteValueInt64(Cycle);
			}

			Serializer.Append(bIsUnknownSpecId ? " ?id=" : " id=");
			Serializer.WriteValueInt64(SpecId);
		}
		else
		{
			++NumEndCpuEvents;

			//Serializer.WriteInteger("E", ActualCycle);
			if (bIsAbsoluteTimestamp)
			{
				Serializer.Append("*E=");
				Serializer.WriteValueInt64(Cycle);
				Serializer.AppendChar('(');
				double Time = Context.EventTime.AsSeconds(Cycle);
				Serializer.WriteValueTime(Time);
				Serializer.AppendChar(')');
			}
			else
			{
				Serializer.Append("E=");
				Serializer.WriteValueInt64(Cycle);
			}
		}
	}
	if (!ensure(BufferPtr == BufferEnd))
	{
		Serializer.Append(" ++");
	}

	*LastCyclePtr = LastCycle;

	Serializer.Append("]");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FConvertToTextAnalyzer::ConvertToTextEventBatchV2(const FOnEventContext& Context, FTextSerializer& Serializer, const uint8* BufferPtr, uint32 BufferSize)
{
	++NumCpuBatches;

	const FEventData& EventData = Context.EventData;

	Serializer.Append("[");

	bool bFirstElement = true;

	const uint32 ThreadId = GetThreadIdField(Context);
	uint64* LastCyclePtr = ThreadMap.Find(ThreadId);
	if (!LastCyclePtr)
	{
		LastCyclePtr = &ThreadMap.Add(ThreadId, 0);
	}
	uint64 LastCycle = *LastCyclePtr;

	const uint8* BufferEnd = BufferPtr + BufferSize;
	while (BufferPtr < BufferEnd)
	{
		if (!bFirstElement)
		{
			Serializer.AppendChar(' ');
		}
		else
		{
			bFirstElement = false;
		}

		uint64 DecodedCycle = Decode7bit(BufferPtr);

		uint64 Cycle = DecodedCycle >> 2;
		uint64 ActualCycle = Cycle + LastCycle;
		LastCycle = ActualCycle;

		if (DecodedCycle & 2ull) // coroutine
		{
			if (DecodedCycle & 1ull) // Restore
			{
				Serializer.Append("coR=");
				Serializer.WriteValueInt64(Cycle);
				uint64 CoroutineId = Decode7bit(BufferPtr);
				Serializer.Append(" coId=");
				Serializer.WriteValueInt64(CoroutineId);
				uint32 TimerScopeDepth = Decode7bit(BufferPtr);
				Serializer.Append(" coDepth=");
				Serializer.WriteValueInt64(TimerScopeDepth);
			}
			else // Save
			{
				Serializer.Append("coS=");
				Serializer.WriteValueInt64(Cycle);
				uint32 TimerScopeDepth = Decode7bit(BufferPtr);
				Serializer.Append(" coDepth=");
				Serializer.WriteValueInt64(TimerScopeDepth);
			}
		}
		else // normal CPU events
		{

			const bool bIsAbsoluteTimestamp = (Cycle > 100000000000ull);

			if (DecodedCycle & 1ull)
			{
				++NumBeginCpuEvents;

				bool bIsUnknownSpecId = false;
				uint32 SpecId = Decode7bit(BufferPtr);
				FString* NamePtr = TimerMap.Find(SpecId);
				if (!NamePtr)
				{
					// SpecId not yet declared (no "EventSpec" event for this id)!
					DefineTimer(SpecId, nullptr);
					bIsUnknownSpecId = true;
				}

				//Serializer.WriteInteger("B", ActualCycle);
				if (bIsAbsoluteTimestamp)
				{
					Serializer.Append("*B=");
					Serializer.WriteValueInt64(Cycle);
					Serializer.AppendChar('(');
					double Time = Context.EventTime.AsSeconds(Cycle);
					Serializer.WriteValueTime(Time);
					Serializer.AppendChar(')');
				}
				else
				{
					Serializer.Append("B=");
					Serializer.WriteValueInt64(Cycle);
				}

				Serializer.Append(bIsUnknownSpecId ? " ?id=" : " id=");
				Serializer.WriteValueInt64(SpecId);
			}
			else
			{
				++NumEndCpuEvents;

				//Serializer.WriteInteger("E", ActualCycle);
				if (bIsAbsoluteTimestamp)
				{
					Serializer.Append("*E=");
					Serializer.WriteValueInt64(Cycle);
					Serializer.AppendChar('(');
					double Time = Context.EventTime.AsSeconds(Cycle);
					Serializer.WriteValueTime(Time);
					Serializer.AppendChar(')');
				}
				else
				{
					Serializer.Append("E=");
					Serializer.WriteValueInt64(Cycle);
				}
			}
		}
	}
	if (!ensure(BufferPtr == BufferEnd))
	{
		Serializer.Append(" ++");
	}

	*LastCyclePtr = LastCycle;

	Serializer.Append("]");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FConvertToTextAnalyzer::DefineTimer(uint32 SpecId, const TCHAR* Name)
{
	TimerMap.Add(SpecId, FString(Name));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FConvertToTextAnalyzer::PrintUnknownTimers(FTextSerializer& Serializer)
{
	TArray<uint32> UnknownTimers;
	for (auto& KV : TimerMap)
	{
		if (KV.Value.IsEmpty())
		{
			UnknownTimers.Add(KV.Key);
		}
	}
	if (UnknownTimers.Num() > 0)
	{
		Serializer.Append("\n");
		Serializer.Appendf("  Unknown CPU Timers: %d / %d [", UnknownTimers.Num(), TimerMap.Num());
		for (uint32 TimerId : UnknownTimers)
		{
			Serializer.Appendf("%u ", TimerId);
		}
		Serializer.Append("]\n");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static void PrintConvertToTextUsage()
{
	puts("Usage:");
	puts("  TraceAnalyzer <in_trace_file> [<options...>]");
	puts("  Where:");
	puts("    <in_trace_file> : the input *.utrace file");
	puts("    -o=<out_text_file> : the output text file; defaults to stdout");
	puts("    -no_new_event_log  : no 'new event' logging");
	puts("    -no_event_log      : no 'event' logging");
	puts("    -no_analysis_stats : no analysis stats");
	puts("    -no_event_stats    : no event stats");
	puts("    -analysis_log_level=<level> : 1=minimum, 2=normal (default), 3=verbose, 4=verbose+");
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Main
////////////////////////////////////////////////////////////////////////////////////////////////////

static int32 ConvertToTextMain(int32 ArgC, TCHAR const* const* ArgV)
{
	if (ArgC < 2)
	{
		PrintConvertToTextUsage();
		return 0;
	}
	const TCHAR* TraceFileName = ArgV[1];

	FileHandle Input = -1;
	FileHandle Output = -1;
	bool bNoNewEventLog = false;
	bool bNoEventLog = false;
	bool bNoAnalysisStats = false;
	bool bNoEventStats = false;
	uint32 AnalysisLogLevel = 2;

	Input = OpenFile(TraceFileName, false);
	if (Input < 0)
	{
		LogTraceAnalyzerError("Cannot open the input trace file ('%s')!", TraceFileName);
		return -2;
	}

	for (int32 ArgIndex = 2; ArgIndex < ArgC; ++ArgIndex)
	{
		if (FCString::Strnicmp(TEXT("-o="), ArgV[ArgIndex], 3) == 0)
		{
			const TCHAR* OutputFileName = ArgV[ArgIndex] + 3;
			Output = OpenFile(OutputFileName, true);
			if (Output < 0)
			{
				LogTraceAnalyzerError("Cannot open the output text file('%s')!", OutputFileName);
				CloseFile(Input);
				return -3;
			}
		}
		else if (FCString::Stricmp(TEXT("-no_new_event_log"), ArgV[ArgIndex]) == 0)
		{
			bNoNewEventLog = true;
		}
		else if (FCString::Stricmp(TEXT("-no_event_log"), ArgV[ArgIndex]) == 0)
		{
			bNoEventLog = true;
		}
		else if (FCString::Stricmp(TEXT("-no_analysis_stats"), ArgV[ArgIndex]) == 0)
		{
			bNoAnalysisStats = true;
		}
		else if (FCString::Stricmp(TEXT("-no_event_stats"), ArgV[ArgIndex]) == 0)
		{
			bNoEventStats = true;
		}
		else if (FCString::Strnicmp(TEXT("-analysis_log_level="), ArgV[ArgIndex], 20) == 0)
		{
			AnalysisLogLevel = FCString::Atoi(ArgV[ArgIndex] + 20);
		}
		else
		{
			LogTraceAnalyzerWarning("Unknown cmd line argument '%s'!", ArgV[ArgIndex]);
		}
	}

#if UE_TRACE_ANALYSIS_DEBUG
	UE::Trace::Analysis::GDebugLevel = AnalysisLogLevel;
#endif

	struct FDataStream : public UE::Trace::IInDataStream
	{
		virtual int32 Read(void* Data, uint32 Size) override
		{
			return FileRead(Handle, Data, Size);
		}

		FileHandle Handle = -1;
	};

	{
		FDataStream DataStream;
		DataStream.Handle = Input;

		{
			FConvertToTextAnalyzer ConvertAnalyzer(Output);
			ConvertAnalyzer.bNoNewEventLog = bNoNewEventLog;
			ConvertAnalyzer.bNoEventLog = bNoEventLog;
			ConvertAnalyzer.bNoAnalysisStats = bNoAnalysisStats;
			ConvertAnalyzer.bNoEventStats = bNoEventStats;
			UE::Trace::FAnalysisContext Context;
			Context.AddAnalyzer(ConvertAnalyzer);
			Context.Process(DataStream).Wait();
		}
	}

	CloseFile(Output);
	CloseFile(Input);

	return 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FCommand GetConvertToTextCommand()
{
	return
	{
		TEXT("conv2text"),
		TEXT("Converts analysis events into a text stream"),
		ConvertToTextMain,
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceAnalyzer
} // namespace UE
