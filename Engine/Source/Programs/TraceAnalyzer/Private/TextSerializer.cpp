// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextSerializer.h"

#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Misc/CString.h"

// TraceAnalysis
#include "TraceAnalysisDebug.h"

#include <limits>
#include <cmath>

namespace UE
{
namespace TraceAnalyzer
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTextSerializer
////////////////////////////////////////////////////////////////////////////////////////////////////

FTextSerializer::FTextSerializer()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTextSerializer::~FTextSerializer()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTextSerializer::WriteValueInt64Auto(int64 Value)
{
	constexpr int64 MinInt64Hex = -99'999'999;
	constexpr int64 MaxInt64Hex = 999'999'999;

	if (Value < MinInt64Hex)
	{
		if (Value == MIN_int64)
		{
			Append("MIN_int64");
		}
		else
		{
			AppendChar('-');
			WriteValueHex64(uint64(-Value));
		}
	}
	else if (Value > MaxInt64Hex)
	{
		if (Value == MAX_int64)
		{
			Append("MAX_int64");
		}
		else
		{
			WriteValueHex64(uint64(Value));
		}
	}
	else
	{
		WriteValueInt64(Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTextSerializer::WriteValueUInt64Auto(uint64 Value)
{
	constexpr uint64 MaxUInt64Hex = 999'999'999;

	if (Value > MaxUInt64Hex)
	{
		if (Value == MAX_uint64)
		{
			Append("MAX_uint64");
		}
		else
		{
			WriteValueHex64(Value);
		}
	}
	else
	{
		WriteValueUInt64(Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTextSerializer::WriteValueFloatAuto(float Value)
{
	if (Value == std::numeric_limits<float>::min())
	{
		Append("MIN_flt");
	}
	else if (Value == std::numeric_limits<float>::max())
	{
		Append("MAX_flt");
	}
	else if (Value == std::numeric_limits<float>::lowest())
	{
		Append("LOWEST_flt");
	}
	else if (Value == std::numeric_limits<float>::denorm_min())
	{
		Append("DENORM_MIN_flt");
	}
	else if (Value == -std::numeric_limits<float>::infinity())
	{
		Append("-INF_flt");
	}
	else if (Value == std::numeric_limits<float>::infinity())
	{
		Append("INF_flt");
	}
	else if (std::isnan(Value))
	{
		Append("NAN_flt");
	}
	else
	{
		Appendf("%g", Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTextSerializer::WriteValueDoubleAuto(double Value)
{
	if (Value == std::numeric_limits<double>::min())
	{
		Append("MIN_dbl");
	}
	else if (Value == std::numeric_limits<double>::max())
	{
		Append("MAX_dbl");
	}
	else if (Value == std::numeric_limits<double>::lowest())
	{
		Append("LOWEST_dbl");
	}
	else if (Value == std::numeric_limits<double>::denorm_min())
	{
		Append("DENORM_MIN_dbl");
	}
	else if (Value == -std::numeric_limits<double>::infinity())
	{
		Append("-INF_dbl");
	}
	else if (Value == std::numeric_limits<double>::infinity())
	{
		Append("INF_dbl");
	}
	else if (std::isnan(Value))
	{
		Append("NAN_dbl");
	}
	else
	{
		Appendf("%g", Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTextSerializer::WriteValueBinary(const void* Data, uint32 Size)
{
	AppendChar('<');
	const uint32 MaxNumElements = 100;
	const uint8* Src = (const uint8*)Data;
	for (uint32 Index = 0; Index < Size; ++Index, ++Src)
	{
		Appendf("%02X", *Src);
		if (Index == MaxNumElements)
		{
			if (Size != MaxNumElements)
			{
				Append("...");
			}
			break;
		}
	}
	if (Size == 1)
	{
		Append(" - 1 byte>");
	}
	else
	{
		Appendf(" - %d bytes>", Size);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FStdoutTextSerializer
////////////////////////////////////////////////////////////////////////////////////////////////////

FStdoutTextSerializer::FStdoutTextSerializer()
{
#if UE_TRACE_ANALYSIS_DEBUG && (UE_TRACE_ANALYSIS_DEBUG_LOG_IMPL <= 2)
	if (UE::Trace::Analysis::GDebugLevel >= 2)
	{
		bWriteEventHeader = false;
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStdoutTextSerializer::AppendChar(const ANSICHAR Value)
{
	printf("%c", Value);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStdoutTextSerializer::Append(const ANSICHAR* Text, int32 Len)
{
	if (Len > 0)
	{
		printf("%.*s", Len, Text);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStdoutTextSerializer::Append(const ANSICHAR* Text)
{
	printf("%s", Text);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStdoutTextSerializer::Appendf(const char* Format, ...)
{
	va_list ArgPtr;
	va_start(ArgPtr, Format);
	vprintf(Format, ArgPtr);
	va_end(ArgPtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FStdoutTextSerializer::Commit()
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFileTextSerializer
////////////////////////////////////////////////////////////////////////////////////////////////////

FFileTextSerializer::FFileTextSerializer(FileHandle InHandle)
	: Handle(InHandle)
{
	Buffer = FMemory::Malloc(BufferSize);
	check(Buffer);

	FormatBuffer = FMemory::Malloc(FormatBufferSize);
	check(FormatBuffer);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FFileTextSerializer::~FFileTextSerializer()
{
	FMemory::Free(FormatBuffer);
	FMemory::Free(Buffer);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileTextSerializer::AppendChar(const ANSICHAR Value)
{
	*(char*)(GetPointer(sizeof(char))) = Value;
	Used += sizeof(char);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileTextSerializer::Append(const ANSICHAR* Text, int32 Len)
{
	if (Len > 0)
	{
		ANSICHAR* Buff = (ANSICHAR*)GetPointer(Len * sizeof(ANSICHAR));
		check(Buff);
		FMemory::Memcpy(Buff, Text, Len * sizeof(ANSICHAR));
		Used += Len * sizeof(ANSICHAR);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileTextSerializer::Append(const ANSICHAR* Text)
{
	int32 Len = FCStringAnsi::Strlen(Text);
	Append(Text, Len);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileTextSerializer::Appendf(const char* Format, ...)
{
	va_list ArgPtr;
	va_start(ArgPtr, Format);
	const int32 Len = FCStringAnsi::GetVarArgs((ANSICHAR*)FormatBuffer, FormatBufferSize / sizeof(ANSICHAR), Format, ArgPtr);
	va_end(ArgPtr);
	check(Len >= 0);
	Append((ANSICHAR*)FormatBuffer, Len);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFileTextSerializer::Commit()
{
	return GetPointer(BufferSize) != nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void* FFileTextSerializer::GetPointer(uint32 RequiredSize)
{
	if (RequiredSize > BufferSize)
	{
		return nullptr;
	}

	uint32 NextUsed = Used + RequiredSize;
	if (NextUsed > BufferSize)
	{
		int BytesOut = FileWrite(Handle, Buffer, Used);
		if (BytesOut < 0 || BytesOut != Used)
		{
			return nullptr;
		}
		Used = 0;
	}

	return (uint8*)Buffer + Used;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceAnalyzer
} // namespace UE
