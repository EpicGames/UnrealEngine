// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceLog
#include "Trace/Trace.h" // for UE::Trace::FEventRef8

// TraceAnalyzer
#include "Io.h"

namespace UE
{
namespace TraceAnalyzer
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTextSerializer
{
public:
	FTextSerializer();
	virtual ~FTextSerializer();

	virtual void AppendChar(const ANSICHAR Value) = 0;
	virtual void Append(const ANSICHAR* Text, int32 Len) = 0;
	virtual void Append(const ANSICHAR* Text) = 0;
	virtual void Appendf(const char* Format, ...) = 0;
	virtual bool Commit() = 0;

	//////////////////////////////////////////////////
	// Attributes

	void BeginAttributeSet()
	{
		AttributeCount = 0;
	}

	void BeginAttribute()
	{
		if (AttributeCount == 0)
		{
			AppendChar('\t');
		}
		else
		{
			AppendChar(' ');
		}
		++AttributeCount;
	}

	void EndAttribute()
	{
	}

	//////////////////////////////////////////////////
	// NEW_EVENT

	void BeginNewEventHeader()
	{
		BeginAttributeSet();
		if (bWriteEventHeader)
		{
#if !UE_TRACE_ANALYSIS_DEBUG
			// Add an empty line before a NEW_EVENT (if previous was an EVENT) to make it more visible.
			if (!bLastWasNewEvent)
			{
				AppendChar('\n');
			}
#endif
			bLastWasNewEvent = true;
			BeginAttribute();
			Append("NEW_EVENT :");
			EndAttribute();
		}
	}
	void EndNewEventHeader()
	{
		if (AttributeCount > 0)
		{
			AppendChar('\n');
		}
	}

	void BeginNewEventFields()
	{
	}
	void BeginField()
	{
		BeginAttributeSet();
		if (bWriteEventHeader)
		{
			AppendChar('\t');
		}
		BeginAttribute();
		Append("FIELD :");
		EndAttribute();
	}
	void EndField()
	{
		AppendChar('\n');
	}
	void EndNewEventFields()
	{
		// Add an extra empty line make each NEW_EVENT more visible.
		AppendChar('\n');
	}

	//////////////////////////////////////////////////
	// EVENT

	bool IsWriteEventHeaderEnabled() const { return bWriteEventHeader; }

	void BeginEvent(uint32 CtxThreadId)
	{
		BeginAttributeSet();
		if (bWriteEventHeader)
		{
			bLastWasNewEvent = false;
			BeginAttribute();
			if (CtxThreadId != (uint32)-1)
			{
				Appendf("EVENT [%u]", CtxThreadId);
			}
			else
			{
				Append("EVENT");
			}
			EndAttribute();
		}
	}

	void WriteEventName(const char* LoggerName, const char* Name)
	{
		if (bWriteEventHeader)
		{
			BeginAttribute();
			Appendf("%s.%s :", LoggerName, Name);
			EndAttribute();
		}
	}

	void EndEvent()
	{
		if (AttributeCount > 0)
		{
			AppendChar('\n');
		}
	}

	//////////////////////////////////////////////////
	// Array: [1 2 3...]

	void BeginArray()
	{
		AppendChar('[');
	}

	void NextArrayElement()
	{
		AppendChar(' ');
	}

	void EndArray()
	{
		AppendChar(']');
	}

	//////////////////////////////////////////////////
	// Values

	void WriteValueString(const char* Value)
	{
		AppendChar('\"');
		Append(Value);
		AppendChar('\"');
	}
	void WriteValueString(const char* Value, uint32 Len)
	{
		AppendChar('\"');
		Append(Value, Len);
		AppendChar('\"');
	}

	void WriteValueReference(const UE::Trace::FEventRef8& Value)  { Appendf("R(%u,%u)", Value.RefTypeId, uint32(Value.Id)); }
	void WriteValueReference(const UE::Trace::FEventRef16& Value) { Appendf("R(%u,%u)", Value.RefTypeId, uint32(Value.Id)); }
	void WriteValueReference(const UE::Trace::FEventRef32& Value) { Appendf("R(%u,%u)", Value.RefTypeId, Value.Id); }
	void WriteValueReference(const UE::Trace::FEventRef64& Value) { Appendf("R(%u,%llu)", Value.RefTypeId, Value.Id); }

	void WriteValueBool(bool Value)         { Append(Value ? "true" : "false"); }

	void WriteValueInt8(int8 Value)         { Appendf("%i", int32(Value)); }
	void WriteValueInt16(int16 Value)       { Appendf("%i", int32(Value)); }
	void WriteValueInt32(int32 Value)       { Appendf("%i", Value); }
	void WriteValueInt64(int64 Value)       { Appendf("%lli", Value); }

	void WriteValueUInt8(uint8 Value)       { Appendf("%u", uint32(Value)); }
	void WriteValueUInt16(uint16 Value)     { Appendf("%u", uint32(Value)); }
	void WriteValueUInt32(uint32 Value)     { Appendf("%u", Value); }
	void WriteValueUInt64(uint64 Value)     { Appendf("%llu", Value); }

	void WriteValueHex8(uint8 Value)        { Appendf("0x%X", uint32(Value)); }
	void WriteValueHex16(uint16 Value)      { Appendf("0x%X", uint32(Value)); }
	void WriteValueHex32(uint32 Value)      { Appendf("0x%X", Value); }
	void WriteValueHex64(uint64 Value)      { Appendf("0x%llX", Value); }

	void WriteValueInt64Auto(int64 Value);
	void WriteValueUInt64Auto(uint64 Value);

	void WriteValueFloat(float Value)       { Appendf("%f", Value); }
	void WriteValueDouble(double Value)     { Appendf("%f", Value); }

	void WriteValueFloatAuto(float Value);
	void WriteValueDoubleAuto(double Value);

	void WriteValueTime(double Time)        { Appendf("%f", Time); }
	void WriteValueNull()                   { Append("null"); }

	void WriteValueBinary(const void* Data, uint32 Size);

	//////////////////////////////////////////////////
	// Key and Values

	void WriteKey(const ANSICHAR* Name)
	{
		Append(Name);
		AppendChar('=');
	}

	void WriteAttributeString(const ANSICHAR* Name, const ANSICHAR* Value)             { BeginAttribute(); WriteKey(Name); WriteValueString(Value);        EndAttribute(); }
	void WriteAttributeString(const ANSICHAR* Name, const ANSICHAR* Value, uint32 Len) { BeginAttribute(); WriteKey(Name); WriteValueString(Value, Len);   EndAttribute(); }
	void WriteAttributeBool(const ANSICHAR* Name, bool Value)                          { BeginAttribute(); WriteKey(Name); WriteValueBool(Value);          EndAttribute(); }
	void WriteAttributeInteger(const ANSICHAR* Name, int64 Value)                      { BeginAttribute(); WriteKey(Name); WriteValueInt64(Value);         EndAttribute(); }
	void WriteAttributeIntegerHex(const ANSICHAR* Name, int64 Value)                   { BeginAttribute(); WriteKey(Name); WriteValueHex64(uint64(Value)); EndAttribute(); }
	void WriteAttributeFloat(const ANSICHAR* Name, float Value)                        { BeginAttribute(); WriteKey(Name); WriteValueFloat(Value);         EndAttribute(); }
	void WriteAttributeDouble(const ANSICHAR* Name, double Value)                      { BeginAttribute(); WriteKey(Name); WriteValueDouble(Value);        EndAttribute(); }
	void WriteAttributeNull(const ANSICHAR* Name)                                      { BeginAttribute(); WriteKey(Name); WriteValueNull();               EndAttribute(); }
	void WriteAttributeBinary(const ANSICHAR* Name, const void* Data, uint32 Size)     { BeginAttribute(); WriteKey(Name); WriteValueBinary(Data, Size);   EndAttribute(); }

protected:
	bool bWriteEventHeader = true;
	bool bLastWasNewEvent = false;
	uint32 AttributeCount = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FStdoutTextSerializer : public FTextSerializer
{
public:
	FStdoutTextSerializer();
	virtual ~FStdoutTextSerializer() {}

	virtual void AppendChar(const ANSICHAR Value) override;
	virtual void Append(const ANSICHAR* Text, int32 Len) override;
	virtual void Append(const ANSICHAR* Text) override;
	virtual void Appendf(const char* Format, ...) override;
	virtual bool Commit() override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFileTextSerializer : public FTextSerializer
{
public:
	FFileTextSerializer(FileHandle InHandle);
	virtual ~FFileTextSerializer();

	virtual void AppendChar(const ANSICHAR Value) override;
	virtual void Append(const ANSICHAR* Text, int32 Len) override;
	virtual void Append(const ANSICHAR* Text) override;
	virtual void Appendf(const char* Format, ...) override;
	virtual bool Commit() override;

private:
	void* GetPointer(uint32 RequiredSize);

private:
	FileHandle Handle = -1;
	static constexpr int BufferSize = 1024 * 1024;
	void* Buffer = nullptr;
	static constexpr int FormatBufferSize = 64 * 1024;
	void* FormatBuffer = nullptr;
	uint32 Used = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceAnalyzer
} // namespace UE
