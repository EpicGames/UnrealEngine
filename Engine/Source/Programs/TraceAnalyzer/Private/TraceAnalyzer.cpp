// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceAnalyzer.h"

#include "Misc/CommandLine.h"
#include "RequiredProgramMainCPPInclude.h"

// TraceAnalyzer
#include "Command.h"

#include <stdio.h>

DEFINE_LOG_CATEGORY(LogTraceAnalyzer);

IMPLEMENT_APPLICATION(TraceAnalyzer, "TraceAnalyzer");

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE
{
namespace TraceAnalyzer
{

FCommand GetConvertToTextCommand();

int32 Main(int32 ArgC, TCHAR const* const* ArgV)
{
	FCommandLine::Set(TEXT(""));
	const FCommand& Cmd = GetConvertToTextCommand();
	return Cmd.Main(ArgC, ArgV);
}

} // namespace TraceAnalyzer
} // namespace UE

////////////////////////////////////////////////////////////////////////////////////////////////////

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
#if 0
	FTaskTagScope Scope(ETaskTag::EGameThread);
	ON_SCOPE_EXIT
	{
		LLM(FLowLevelMemTracker::Get().UpdateStatsPerFrame());
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};

	if (int32 Ret = GEngineLoop.PreInit(ArgC, ArgV))
	{
		return Ret;
	}

	UE_LOGF(LogTraceAnalyzer, Display, "Trace Analyzer");
#endif

	return UE::TraceAnalyzer::Main(ArgC, ArgV);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////
