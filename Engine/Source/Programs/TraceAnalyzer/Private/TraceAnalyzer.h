// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogCategory.h"

DECLARE_LOG_CATEGORY_EXTERN(LogTraceAnalyzer, Log, All);

#define LogTraceAnalyzerError(Format, ...) \
	{ \
		wprintf(L"Error: " Format "\n", ##__VA_ARGS__); \
		UE_LOG(LogTraceAnalyzer, Error, TEXT(Format), ##__VA_ARGS__); \
	}
#define LogTraceAnalyzerWarning(Format, ...) \
	{ \
		wprintf(L"Warning: " Format "\n", ##__VA_ARGS__); \
		UE_LOG(LogTraceAnalyzer, Warning, TEXT(Format), ##__VA_ARGS__); \
	}
#define LogTraceAnalyzerMessage(Format, ...) \
	{ \
		wprintf(L"" Format "\n", ##__VA_ARGS__); \
		UE_LOG(LogTraceAnalyzer, Log, TEXT(Format), ##__VA_ARGS__); \
	}
