// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TraceAnalyzer : ModuleRules
{
	public TraceAnalyzer(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("Projects");
		PrivateDependencyModuleNames.Add("TraceAnalysis");
	}
}
