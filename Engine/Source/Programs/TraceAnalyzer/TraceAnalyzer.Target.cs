// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class TraceAnalyzerTarget : TargetRules
{
	public TraceAnalyzerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LaunchModuleName = "TraceAnalyzer";
		LinkType = TargetLinkType.Monolithic;

		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		// This app is a console application, not a Windows app (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;

		// Lean and mean
		bBuildDeveloperTools = false;

		bBuildWithEditorOnlyData = false;

		//bUseStaticCRT = true;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstApplicationCore = false;
		bCompileICU = false;
		//bCompileCEF3 = false;

		bEnableTrace = false; // UE_TRACE_ENABLED
		GlobalDefinitions.Add("CPUPROFILERTRACE_ENABLED=0");
		GlobalDefinitions.Add("LOADTIMEPROFILERTRACE_ENABLED=0");
		GlobalDefinitions.Add("STATSTRACE_ENABLED=0");
		GlobalDefinitions.Add("COUNTERSTRACE_ENABLED=0");
		GlobalDefinitions.Add("LOGTRACE_ENABLED=0");
		GlobalDefinitions.Add("MISCTRACE_ENABLED=0");
		GlobalDefinitions.Add("UE_TASK_TRACE_ENABLED=0");
		GlobalDefinitions.Add("UE_NET_TRACE_ENABLED=0");
		GlobalDefinitions.Add("UE_MEMORY_TAGS_TRACE_ENABLED=0");
		GlobalDefinitions.Add("UE_MEMORY_TRACE_ENABLED=0");
		GlobalDefinitions.Add("UE_CALLSTACK_TRACE_ENABLED=0");

		// see Engine\Source\Developer\TraceAnalysis\Public\TraceAnalysisDebug.h
		GlobalDefinitions.Add("UE_TRACE_ANALYSIS_DEBUG_API=1");
		GlobalDefinitions.Add("UE_TRACE_ANALYSIS_DEBUG=1");
		GlobalDefinitions.Add("UE_TRACE_ANALYSIS_DEBUG_LOG_IMPL=1");

		//GlobalDefinitions.Add("_CRT_SECURE_NO_WARNINGS");
		//GlobalDefinitions.Add("_CRT_NONSTDC_NO_WARNINGS");
		//WindowsPlatform.TargetWindowsVersion = 0x0603; // Windows 8.1
	}
}
